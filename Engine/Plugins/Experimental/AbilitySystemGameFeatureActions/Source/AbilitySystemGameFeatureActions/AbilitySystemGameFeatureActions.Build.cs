// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AbilitySystemGameFeatureActions : ModuleRules
	{
        public AbilitySystemGameFeatureActions(ReadOnlyTargetRules Target) : base(Target)
		{
			ShortName = "AbilitySysGFAs";

			PublicDependencyModuleNames.AddRange(
				new string[]
                {
                    "Core",
                    "CoreUObject",
					"DeveloperSettings",
					"Engine",
                    "ModularGameplay",
					"DataRegistry"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"GameFeatures",
					"GameplayAbilities",
				}
			);
		}
	}
}
