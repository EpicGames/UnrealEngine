// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureAction_AddAttributeDefaults.h"
#include "AbilitySystemGlobals.h"
#include "HAL/IConsoleManager.h"
#include "Misc/PackageName.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeatureAction_AddAttributeDefaults)

#define LOCTEXT_NAMESPACE "GameFeatures"

namespace GameFeatureAction_AddAttributeDefaults
{
	static TAutoConsoleVariable<bool> CVarAllowRemoveAttributeDefaultTables(TEXT("GameFeatureAction_AddAttributeDefaults.AllowRemoveAttributeDefaultTables"), true, TEXT("Removes hard references when unregistering"));
}

//////////////////////////////////////////////////////////////////////
// UGameFeatureAction_AddAttributeDefaults

void UGameFeatureAction_AddAttributeDefaults::OnGameFeatureRegistering()
{
	Super::OnGameFeatureRegistering();

	if (ShouldAddAttributeDefaults())
	{
		AddAttributeDefaults();
	}
}

void UGameFeatureAction_AddAttributeDefaults::OnGameFeatureActivating(FGameFeatureActivatingContext& Context)
{
	Super::OnGameFeatureActivating(Context);

	if (ShouldAddAttributeDefaults())
	{
		AddAttributeDefaults();
	}
}

void UGameFeatureAction_AddAttributeDefaults::OnGameFeatureUnregistering()
{
	if (ShouldRemoveAttributeDefaults())
	{
		RemoveAttributeDefaults();
	}

	Super::OnGameFeatureUnregistering();
}

void UGameFeatureAction_AddAttributeDefaults::OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context)
{
	if (ShouldRemoveAttributeDefaults())
	{
		RemoveAttributeDefaults();
	}

	Super::OnGameFeatureDeactivating(Context);
}

#if WITH_EDITOR
void UGameFeatureAction_AddAttributeDefaults::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, bApplyOnRegister))
	{
		// Re-check whether we should apply our modified defaults.
		// Avoids 'leaking' changes in the event the GFA is unregistered.

		if (ShouldRemoveAttributeDefaults())
		{
			RemoveAttributeDefaults();
		}

		if (ShouldAddAttributeDefaults())
		{
			AddAttributeDefaults();
		}
	}
}
#endif // WITH_EDITOR

bool UGameFeatureAction_AddAttributeDefaults::ShouldAddAttributeDefaults() const
{
	// Necessary as during OnGameFeatureRegistering/Activating the plugin is *Registering* but not *Registered*
	constexpr bool bCheckForRegisteringAndActivating = true;

	if (!bAttributesHaveBeenSet)
	{
		return bApplyOnRegister
			? IsGameFeaturePluginRegistered(bCheckForRegisteringAndActivating)
			: IsGameFeaturePluginActive(bCheckForRegisteringAndActivating);
	}

	return false;
}

bool UGameFeatureAction_AddAttributeDefaults::ShouldRemoveAttributeDefaults() const
{
	if (bAttributesHaveBeenSet)
	{
		return bApplyOnRegister
			? !IsGameFeaturePluginRegistered()
			: !IsGameFeaturePluginActive();
	}

	return false;
}

void UGameFeatureAction_AddAttributeDefaults::AddAttributeDefaults()
{
	const TArray<FSoftObjectPath>* AttribDefaultTableNamesToAdd = &AttribDefaultTableNames;

#if WITH_EDITOR
	// Do a file exists check in editor builds since some folks do not sync all data in the editor. Ideally we don't need to load anything at GFD registration time, but for now we will do this.
	TArray<FSoftObjectPath> AttribDefaultTableNamesThatExist;
	AttribDefaultTableNamesToAdd = &AttribDefaultTableNamesThatExist;
	for (const FSoftObjectPath& Path : AttribDefaultTableNames)
	{
		if (FPackageName::DoesPackageExist(Path.GetLongPackageName()))
		{
			AttribDefaultTableNamesThatExist.Add(Path);
		}
	}
#endif // WITH_EDITOR

	if (!AttribDefaultTableNamesToAdd->IsEmpty())
	{
		FNameBuilder OwnerNameBuilder;
		GetPathName(nullptr, OwnerNameBuilder);
		AttributeDefaultTablesOwnerName = FName(OwnerNameBuilder.ToView());

		UAbilitySystemGlobals& AbilitySystemGlobals = UAbilitySystemGlobals::Get();
		AbilitySystemGlobals.AddAttributeDefaultTables(AttributeDefaultTablesOwnerName, *AttribDefaultTableNamesToAdd);
	}

	bAttributesHaveBeenSet = true;
}

void UGameFeatureAction_AddAttributeDefaults::RemoveAttributeDefaults()
{
	if (!AttribDefaultTableNames.IsEmpty() && GameFeatureAction_AddAttributeDefaults::CVarAllowRemoveAttributeDefaultTables.GetValueOnAnyThread())
	{
		UAbilitySystemGlobals& AbilitySystemGlobals = UAbilitySystemGlobals::Get();
		AbilitySystemGlobals.RemoveAttributeDefaultTables(AttributeDefaultTablesOwnerName, AttribDefaultTableNames);
	}

	bAttributesHaveBeenSet = false;
}

//////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

