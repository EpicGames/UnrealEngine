// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFeatureAction.h"

#include "GameFeatureAction_AddAttributeDefaults.generated.h"

//////////////////////////////////////////////////////////////////////
// UGameFeatureAction_AddAttributeDefaults

/**
 * Adds ability system attribute defaults from this game feature
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Add Attribute Defaults"))
class UGameFeatureAction_AddAttributeDefaults final : public UGameFeatureAction
{
	GENERATED_BODY()

public:
	//~UGameFeatureAction interface
	virtual void OnGameFeatureRegistering() override;
	virtual void OnGameFeatureActivating(FGameFeatureActivatingContext& Context) override;
	virtual void OnGameFeatureUnregistering() override;
	virtual void OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context) override;
	//~End of UGameFeatureAction interface

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	/**
	 * True: Apply defaults when the game feature is registered.
	 * False: Apply defaults when the game feature is activated.
	 */
	UPROPERTY(EditAnywhere, Category = Attributes, AdvancedDisplay)
	bool bApplyOnRegister = true;

	/** List of attribute default tables to add */
	UPROPERTY(EditAnywhere, Category = Attributes)
	TArray<FSoftObjectPath> AttribDefaultTableNames;

private:

	bool ShouldAddAttributeDefaults() const;
	bool ShouldRemoveAttributeDefaults() const;

	void AddAttributeDefaults();
	void RemoveAttributeDefaults();

	FName AttributeDefaultTablesOwnerName;

	bool bAttributesHaveBeenSet = false;
};
